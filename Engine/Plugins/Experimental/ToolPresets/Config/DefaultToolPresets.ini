[CoreRedirects]
+PackageRedirects=(OldName="/Script/PresetEditor",NewName="/Script/ToolPresetEditor")
+ClassRedirects=(OldName="/Script/PresetEditor.PresetUserSettings",NewName="/Script/ToolPresetEditor.ToolPresetUserSettings")
+ClassRedirects=(OldName="/Script/PresetEditor.PresetProjectSettings",NewName="/Script/ToolPresetEditor.ToolPresetProjectSettings")

+PackageRedirects=(OldName="/Script/PresetAsset",NewName="/Script/ToolPresetAsset")
+StructRedirects=(OldName="/Script/PresetAsset.InteractiveToolPresetDefintion",NewName="/Script/ToolPresetAsset.InteractiveToolPresetDefinition")
+StructRedirects=(OldName="/Script/PresetAsset.InteractiveToolPresetStore",NewName="/Script/ToolPresetAsset.InteractiveToolPresetStore")
+ClassRedirects=(OldName="/Script/PresetAsset.InteractiveToolsPresetCollectionAsset",NewName="/Script/ToolPresetAsset.InteractiveToolsPresetCollectionAsset")
+ClassRedirects=(OldName="/Script/PresetAsset.InteractiveToolsPresetCollectionAssetFactory",NewName="/Script/ToolPresetAsset.InteractiveToolsPresetCollectionAssetFactory")
+ClassRedirects=(OldName="/Script/PresetAsset.PresetAssetSubsystem",NewName="/Script/ToolPresetAsset.ToolPresetAssetSubsystem")
