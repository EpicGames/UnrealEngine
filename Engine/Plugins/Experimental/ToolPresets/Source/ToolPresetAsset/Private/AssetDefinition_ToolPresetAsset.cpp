// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_ToolPresetAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_ToolPresetAsset)

#define LOCTEXT_NAMESPACE "AssetDefinition_ToolPresetAsset"

#undef LOCTEXT_NAMESPACE
