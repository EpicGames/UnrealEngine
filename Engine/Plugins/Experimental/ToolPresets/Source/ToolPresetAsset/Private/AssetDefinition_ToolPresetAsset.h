// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolPresetAsset.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_ToolPresetAsset.generated.h"

enum class EAssetCommandResult : uint8;
struct FAssetCategoryPath;
struct FAssetOpenArgs;

UCLASS()
class UAssetDefinition_InteractiveToolsPresetCollectionAsset : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_InteractiveToolsPresetCollectionAsset", "Tool Preset"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(175, 0, 128)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UInteractiveToolsPresetCollectionAsset::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Misc, NSLOCTEXT("AssetDefinition", "AssetDefinition_InteractiveToolsPresetCollectionAssetSubMenu", "Other"), ECategoryMenuType::Section)
			};
		return Categories;
	}
	virtual FText GetObjectDisplayNameText(UObject* Object) const override { return FText::FromString(TEXT("UInteractiveToolsPresetCollectionAsset")); }
	// UAssetDefinition End
};
