// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolPresetAsset.h"
#include "JsonObjectConverter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolPresetAsset)

void FInteractiveToolPresetDefinition::SetStoredPropertyData(TArray<UObject*>& Properties)
{
	FJsonObjectWrapper JsonWrapper;

	for (UObject* PropertySet : Properties)
	{
		TSharedPtr<FJsonObject> PropertyJsonObject = MakeShared<FJsonObject>();

		if (PropertyJsonObject)
		{
			for (FProperty* Prop : TFieldRange<FProperty>(PropertySet->GetClass()))
			{
#if WITH_EDITOR
				if (! (Prop->HasMetaData(TEXT("TransientToolProperty")) ||
					   Prop->GetPropertyFlags() & EPropertyFlags::CPF_SkipSerialization ||
					   Prop->GetPropertyFlags() & EPropertyFlags::CPF_Transient) )
#endif
				{
					TSharedPtr<FJsonValue> JsonProp = FJsonObjectConverter::UPropertyToJsonValue(Prop, Prop->ContainerPtrToValuePtr<void>(PropertySet));
					FString FieldName;
					Prop->GetName(FieldName);
					PropertyJsonObject->SetField(FieldName, JsonProp);
				}
			}			
		}

		FString ClassName;
		PropertySet->GetClass()->GetName(ClassName);
		JsonWrapper.JsonObject->SetObjectField(ClassName, PropertyJsonObject);
	}
	
	JsonWrapper.JsonObjectToString(StoredProperties);
}

void FInteractiveToolPresetDefinition::LoadStoredPropertyData(TArray<UObject*>& Properties)
{
	FJsonObjectWrapper JsonWrapper;
	JsonWrapper.JsonObjectFromString(StoredProperties);
	
	for (UObject* PropertySet : Properties)
	{
		FString ClassName;
		PropertySet->GetClass()->GetName(ClassName);
		TSharedPtr<FJsonObject> PropertyJsonObject = JsonWrapper.JsonObject->GetObjectField(ClassName);

		if (PropertyJsonObject)
		{
			for (FProperty* Prop : TFieldRange<FProperty>(PropertySet->GetClass()))
			{
#if WITH_EDITOR
				if (!(Prop->HasMetaData(TEXT("TransientToolProperty")) ||
					Prop->GetPropertyFlags() & EPropertyFlags::CPF_SkipSerialization ||
					Prop->GetPropertyFlags() & EPropertyFlags::CPF_Transient))
#endif
				{
					FString FieldName;
					Prop->GetName(FieldName);
					const TSharedPtr<FJsonValue> JsonField = PropertyJsonObject->GetFieldUntyped(FieldName);
					if (JsonField.IsValid())
					{
						if (FJsonObjectConverter::JsonValueToUProperty(JsonField, Prop, Prop->ContainerPtrToValuePtr<void>(PropertySet)))
						{
							FPropertyChangedEvent ChangeEvent(Prop, EPropertyChangeType::ValueSet);
							PropertySet->PostEditChangeProperty(ChangeEvent);
						}
					}
				}
			}
		}
	}
}

bool FInteractiveToolPresetDefinition::IsValid() const
{
	FJsonObjectWrapper JsonWrapper;
	return JsonWrapper.JsonObjectFromString(StoredProperties);
}
