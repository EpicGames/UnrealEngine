// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"

#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ToolPresetAsset.h"

#define LOCTEXT_NAMESPACE "FToolPresetAssetModule"

/**
 * Implements the PresetAsset module.
 */
class FToolPresetAssetModule
	: public IModuleInterface
{
public:

	//~ IModuleInterface interface

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual void OnPostEngineInit();
	virtual bool SupportsDynamicReloading() override
	{
		return true;
	}

protected:

	TArray<FName> ClassesToUnregisterOnShutdown;
};


// This function will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
void FToolPresetAssetModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FToolPresetAssetModule::OnPostEngineInit);
}

// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
// we call this function before unloading the module.
void FToolPresetAssetModule::ShutdownModule()
{
	// Unregister customizations
	FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyEditorModule)
	{
		for (FName ClassName : ClassesToUnregisterOnShutdown)
		{
			PropertyEditorModule->UnregisterCustomClassLayout(ClassName);
		}
	}
}

void FToolPresetAssetModule::OnPostEngineInit()
{
	ClassesToUnregisterOnShutdown.Reset();

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
}


IMPLEMENT_MODULE(FToolPresetAssetModule, ToolPresetAsset);

#undef LOCTEXT_NAMESPACE
