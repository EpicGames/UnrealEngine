// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolPresetAssetSubsystem.h"

#include "EditorConfigSubsystem.h"
#include "ToolPresetAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolPresetAssetSubsystem)

#define LOCTEXT_NAMESPACE "ToolPresetAssetSubsystem"

void UToolPresetAssetSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Collection.InitializeDependency(UEditorConfigSubsystem::StaticClass());

	InitializeDefaultCollection();
}

void UToolPresetAssetSubsystem::Deinitialize()
{
	if (DefaultCollection)
	{
		DefaultCollection->SaveEditorConfig();
		DefaultCollection = nullptr;
	}	
}

UInteractiveToolsPresetCollectionAsset* UToolPresetAssetSubsystem::GetDefaultCollection()
{
	return DefaultCollection;
}

bool UToolPresetAssetSubsystem::SaveDefaultCollection()
{
	if (DefaultCollection)
	{
		return DefaultCollection->SaveEditorConfig();
	}
	return false;
}


void UToolPresetAssetSubsystem::InitializeDefaultCollection()
{
	/*
	* We're storing the default collection as a JSON file instead of an asset
	* on disk for a few reasons. First it avoids issues around automatically
	* creating assets, both from avoiding build system issues and from a more
	* philosophical point about requiring user involvement. Second, it helps
	* compartmentalizing the "default" collection as more of Editor preferences,
	* rather than a specific collection which has purpose and can be shared around. 
	*/

	DefaultCollection = NewObject<UInteractiveToolsPresetCollectionAsset>();
	DefaultCollection->CollectionLabel = LOCTEXT("DefaultCollectionLabel", "Personal Tool Presets (Default)");
	DefaultCollection->LoadEditorConfig();
}

#undef LOCTEXT_NAMESPACE
