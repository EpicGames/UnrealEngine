// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//#include "Internationalization/Text.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "AssetToolsModule.h"
#include "EditorConfigBase.h"

#include "ToolPresetAsset.generated.h"

#define UE_API TOOLPRESETASSET_API

#define LOCTEXT_NAMESPACE "ToolPresetAsset"

USTRUCT(BlueprintType)
struct FInteractiveToolPresetDefinition
{
	GENERATED_BODY()
public:
	UPROPERTY(VisibleAnywhere, Category = "ToolPresetAsset", meta = (EditorConfig))
	FString StoredProperties;

	UPROPERTY(EditAnywhere, Category = "ToolPresetAsset", meta = (EditorConfig))
	FString Label;

	UPROPERTY(EditAnywhere, Category = "ToolPresetAsset", meta = (EditorConfig))
	FString Tooltip;

	UE_API bool IsValid() const;
	UE_API void SetStoredPropertyData(TArray<UObject*>& Properties);
	UE_API void LoadStoredPropertyData(TArray<UObject*>& Properties);
};

USTRUCT(BlueprintType)
struct FInteractiveToolPresetStore
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "ToolPresetAsset", meta = (EditorConfig))
	TArray<FInteractiveToolPresetDefinition> NamedPresets;

	UPROPERTY(VisibleAnywhere, Category = "ToolPresetAsset", meta = (EditorConfig))
	FText ToolLabel;

	UPROPERTY(VisibleAnywhere, Category = "ToolPresetAsset", meta = (EditorConfig))
	FSlateBrush ToolIcon;
};

/**
 * Implements an asset that can be used to store tool settings as a named preset
 */
UCLASS(MinimalAPI, BlueprintType, hidecategories=(Object), EditorConfig = "UInteractiveToolsPresetCollectionAsset_DefaultCollection")
class UInteractiveToolsPresetCollectionAsset 
	: public UEditorConfigBase
{
	GENERATED_BODY()

public:

	// TODO: Currently there are no helper methods within this class, simply providing
	// raw access to the underlying arrays and maps. This is intentional.
	// Until the design of the preset concept is more firmly decided, it seems like a waste to
	// implement a bunch of methods that we don't know if we actually want/need at the end.
	// Once we are satisifed with the data structure, planned accessors and mutators will include
	// support for adding, removing, renaming, saving and retrieving presets.

	UPROPERTY(VisibleAnywhere, Category = "ToolPresetAsset", meta = (EditorConfig) )
	TMap<FString, FInteractiveToolPresetStore > PerToolPresets;

	UPROPERTY(EditAnywhere, Category = "ToolPresetAsset", meta = (EditorConfig))
	FText CollectionLabel;
};


UCLASS(MinimalAPI, hidecategories = Object)
class UInteractiveToolsPresetCollectionAssetFactory : public UFactory
{
	GENERATED_BODY()

public:

	UInteractiveToolsPresetCollectionAssetFactory()
	{
		SupportedClass = UInteractiveToolsPresetCollectionAsset::StaticClass();

		bCreateNew = true;
		bEditAfterNew = true;
	}

	virtual FText GetToolTip() const override
	{
		return LOCTEXT("Tooltip", "Tool Presets capture the state of tool settings for later reloading.");
	}

	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override
	{
		UInteractiveToolsPresetCollectionAsset* Preset = nullptr;
		if (ensure(SupportedClass == Class))
		{
			ensure(0 != (RF_Public & Flags));
			Preset = NewObject<UInteractiveToolsPresetCollectionAsset>(InParent, Class, Name, Flags);
			Preset->CollectionLabel = FText::FromName(Name);
		}
		return Preset;
	}

	virtual FString GetDefaultNewAssetName() const override
	{
		return TEXT("Tool Preset");
	}

	virtual bool ShouldShowInNewMenu() const override { return true; }
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
