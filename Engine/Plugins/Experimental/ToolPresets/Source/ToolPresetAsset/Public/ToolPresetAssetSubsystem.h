// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorSubsystem.h"
#include "ToolPresetAssetSubsystem.generated.h"

#define UE_API TOOLPRESETASSET_API

class UInteractiveToolsPresetCollectionAsset;

/**
 * Using an editor subsystem allows us to make sure that we have a default preset asset whenever the editor exists
 *  (and to avoid accidentally trying to make one when it doesn't, such as when running cooking scripts).
 */
UCLASS(MinimalAPI)
class UToolPresetAssetSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;

	UE_API UInteractiveToolsPresetCollectionAsset* GetDefaultCollection();
	UE_API bool SaveDefaultCollection();


protected:
	UE_API void InitializeDefaultCollection();

	UPROPERTY()
	TObjectPtr<UInteractiveToolsPresetCollectionAsset> DefaultCollection;
};

#undef UE_API
