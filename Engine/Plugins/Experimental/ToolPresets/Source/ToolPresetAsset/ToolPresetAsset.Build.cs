// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ToolPresetAsset: ModuleRules
{
	public ToolPresetAsset(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",				
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetDefinition",
				"CoreUObject",
				"EditorConfig",
				"EditorSubsystem",
				"Engine",
				"Json",
				"JsonUtilities",
				"Slate",
				"SlateCore",
				"UnrealEd",
			});
	}
}
