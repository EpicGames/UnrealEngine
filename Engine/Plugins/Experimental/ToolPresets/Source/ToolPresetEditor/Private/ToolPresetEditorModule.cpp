// Copyright Epic Games, Inc. All Rights Reserved.

#include "IToolPresetEditorModule.h"

#include "Delegates/Delegate.h"
#include "Framework/Docking/TabManager.h"
#include "Internationalization/Internationalization.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "ToolPresetEditorStyle.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SToolPresetManager.h"

#define LOCTEXT_NAMESPACE "ToolPresetEditorModule"

const FName PresetEditorTabName("ToolPreset");

class FToolPresetEditorModule : public IToolPresetEditorModule
{
public:
	virtual void StartupModule() override
	{
		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(PresetEditorTabName, FOnSpawnTab::CreateRaw(this, &FToolPresetEditorModule::HandleSpawnPresetEditorTab))
			.SetDisplayName(NSLOCTEXT("FToolPresetModule", "ToolPresetTabTitle", "Tool Preset Manager"))
			.SetTooltipText(NSLOCTEXT("FToolPresetModule", "ToolPresetTooltipText", "Open the Tool Preset Manager tab."))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ToolPreset.TabIcon"))
			.SetAutoGenerateMenuEntry(false);

		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FToolPresetEditorModule::OnPostEngineInit);
	}
	
	virtual void ShutdownModule() override
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(PresetEditorTabName);

		FToolPresetEditorStyle::Shutdown();
	}

	void OnPostEngineInit()
	{
		// Register slate style overrides
		FToolPresetEditorStyle::Initialize();
	}

	virtual bool SupportsDynamicReloading() override
	{
		return true;
	}

	virtual void ExecuteOpenPresetEditor() override
	{
		FGlobalTabmanager::Get()->TryInvokeTab(PresetEditorTabName);
	}	

private:	
	/** Handles creating the project settings tab. */
	TSharedRef<SDockTab> HandleSpawnPresetEditorTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
		[
			SNew(SToolPresetManager)
		];
		return DockTab;
	}
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FToolPresetEditorModule, ToolPresetEditor);
