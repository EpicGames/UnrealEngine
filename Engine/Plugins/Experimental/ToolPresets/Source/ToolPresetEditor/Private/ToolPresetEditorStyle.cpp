// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolPresetEditorStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "SlateOptMacros.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FToolPresetEditorStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )
#define RootToContentDir StyleSet->RootToContentDir

FString FToolPresetEditorStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("ToolPresets"))->GetContentDir();
	return (ContentDir / RelativePath) + Extension;
}

TSharedPtr< FSlateStyleSet > FToolPresetEditorStyle::StyleSet = nullptr;
TSharedPtr< class ISlateStyle > FToolPresetEditorStyle::Get() { return StyleSet; }

FName FToolPresetEditorStyle::GetStyleSetName()
{
	static FName PresetEditorStyleName(TEXT("ToolPresetEditorStyle"));
	return PresetEditorStyleName;
}

const FSlateBrush* FToolPresetEditorStyle::GetBrush(FName PropertyName, const ANSICHAR* Specifier)
{
	return Get()->GetBrush(PropertyName, Specifier);
}


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FToolPresetEditorStyle::Initialize()
{
	// Const icon sizes
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon28x28(28.0f, 28.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon120(120.0f, 120.0f);

	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet(GetStyleSetName()));

	// If we get asked for something that we don't set, we should default to editor style
	StyleSet->SetParentStyleName("EditorStyle");

	StyleSet->SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/ToolPresets/Content"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// Icons
	{
		StyleSet->Set("ManagerIcons.Tools", new IMAGE_PLUGIN_BRUSH("Icons/Primitive_40x", Icon40x40));


    }

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef IMAGE_PLUGIN_BRUSH
#undef RootToContentDir

void FToolPresetEditorStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}
