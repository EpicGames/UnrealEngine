// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolPresetSettings.h"

#include "UObject/ObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolPresetSettings)

TObjectPtr<UToolPresetUserSettings> UToolPresetUserSettings::Instance = nullptr;

void UToolPresetUserSettings::Initialize()
{
	if (Instance == nullptr)
	{
		Instance = NewObject<UToolPresetUserSettings>();
		Instance->AddToRoot();
	}
}

UToolPresetUserSettings* UToolPresetUserSettings::Get()
{
	return Instance;
}
