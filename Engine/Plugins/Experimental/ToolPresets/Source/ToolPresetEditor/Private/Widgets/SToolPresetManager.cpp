// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SToolPresetManager.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IAssetTools.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"
#include "ObjectTools.h"
#include "SlateOptMacros.h"
#include "SNegativeActionButton.h"
#include "SPositiveActionButton.h"
#include "SSimpleButton.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "ToolPresetAsset.h"
#include "ToolPresetAssetSubsystem.h"
#include "ToolPresetEditorStyle.h"
#include "ToolPresetSettings.h"
#include "UObject/SavePackage.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "PackageTools.h"

#define LOCTEXT_NAMESPACE "SToolPresetManager"

DECLARE_DELEGATE_TwoParams(FOnCollectionEnabledCheckboxChanged, TSharedPtr<SToolPresetManager::FToolPresetViewEntry>, ECheckBoxState)
DECLARE_DELEGATE_TwoParams(FOnPresetLabelChanged, TSharedPtr<SToolPresetManager::FToolPresetViewEntry>, FText)
DECLARE_DELEGATE_TwoParams(FOnPresetTooltipChanged, TSharedPtr<SToolPresetManager::FToolPresetViewEntry>, FText)
DECLARE_DELEGATE_OneParam(FOnPresetDeleted, TSharedPtr<SToolPresetManager::FToolPresetViewEntry>)
DECLARE_DELEGATE_TwoParams(FOnCollectionRenameStarted, TSharedPtr<SToolPresetManager::FToolPresetViewEntry>, TSharedPtr<SEditableTextBox> RenameWidget)
DECLARE_DELEGATE_TwoParams(FOnCollectionRenameEnded, TSharedPtr<SToolPresetManager::FToolPresetViewEntry>, const FText& NewText)

namespace UE::ToolPresetEditor::Private
{
	template <typename AssetType>
	void GetObjectsOfClass(TArray<FSoftObjectPath>& OutArray)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		TArray<FAssetData> AssetData;

		FARFilter Filter;
		Filter.ClassPaths.Add(AssetType::StaticClass()->GetClassPathName());
		Filter.PackagePaths.Add(FName("/ToolPresets"));
		Filter.bRecursiveClasses = false;	
		Filter.bRecursivePaths = true;
		Filter.bIncludeOnlyOnDiskAssets = false;
		
		AssetRegistryModule.Get().GetAssets(Filter, AssetData);

		for (int i = 0; i < AssetData.Num(); i++)
		{
			AssetType* Object = Cast<AssetType>(AssetData[i].GetAsset());		
			if (Object)
			{				
				OutArray.Add(Object->GetPathName());
			}
		}
	}

	template<typename ItemType> class SCollectionTableRow;

	template<typename ItemType>
	class SCollectionTableRow : public STableRow< ItemType>
	{
		typedef SCollectionTableRow< ItemType > FSuperRowType;
		typedef typename STableRow<ItemType>::FArguments FTableRowArgs;
		typedef SToolPresetManager::FToolPresetViewEntry::EEntryType EEntryType;

	public:

		SLATE_BEGIN_ARGS(SCollectionTableRow) { }
		    SLATE_ARGUMENT(TSharedPtr<SToolPresetManager::FToolPresetViewEntry>, ViewEntry)
			SLATE_EVENT(FOnCollectionEnabledCheckboxChanged, OnCollectionEnabledCheckboxChanged)
			SLATE_EVENT(FOnCollectionRenameStarted, OnCollectionRenameStarted)
			SLATE_EVENT(FOnCollectionRenameEnded, OnCollectionRenameEnded)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
		{
			FTableRowArgs Args = FTableRowArgs()
				.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"))
				.ExpanderStyleSet(& FCoreStyle::Get());

			ViewEntry = InArgs._ViewEntry;
			OnCollectionEnabledCheckboxChanged = InArgs._OnCollectionEnabledCheckboxChanged;
			OnCollectionRenameStarted = InArgs._OnCollectionRenameStarted;
			OnCollectionRenameEnded = InArgs._OnCollectionRenameEnded;

			STableRow<ItemType>::Construct(Args, InOwnerTableView);
		}

		BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
		virtual void ConstructChildren(ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent) override
		{
			STableRow<ItemType>::Content = InContent;

			TSharedPtr<class ITableRow> ThisTableRow = this->SharedThis(this);

			if(InOwnerTableMode == ETableViewMode::Tree)
			{

				// Rows in a TreeView need an expander button and some indentation
				this->ChildSlot
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Fill)
					[
						SAssignNew(EnabledWidget, SCheckBox)
						.Visibility_Lambda([this]()
						{
							return ViewEntry->EntryType == EEntryType::Collection ? EVisibility::Visible : EVisibility::Collapsed;
						})
						.IsChecked_Lambda([this]()
						{
							return ViewEntry->bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						})
						.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
						{
							if (OnCollectionEnabledCheckboxChanged.IsBound())
							{
								OnCollectionEnabledCheckboxChanged.Execute(ViewEntry, State);
							}
						})
				
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Fill)
					[
						SAssignNew(STableRow< ItemType>::ExpanderArrowWidget, SExpanderArrow, ThisTableRow)
						.StyleSet(STableRow< ItemType>::ExpanderStyleSet)
						.ShouldDrawWires(false)
					]

					+ SHorizontalBox::Slot()
						.AutoWidth()						
						.Padding(5.0f)
						[
							SNew(SImage)
							.Visibility_Lambda([this]()
							{
								return ViewEntry->EntryType == EEntryType::Tool ? EVisibility::Visible : EVisibility::Collapsed;
							})
							.Image(&ViewEntry->EntryIcon)
						]

					+ SHorizontalBox::Slot()
						.FillWidth(1)
						.Padding(5.0f)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							[
								SNew(STextBlock)
								.Text(ViewEntry->EntryLabel)
								.Visibility_Lambda([this]()
								{
									return ViewEntry->bIsRenaming ? EVisibility::Collapsed : EVisibility::Visible;
								})							
								.Font_Lambda([this]()
								{
									return FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText").Font;							
								})
							]

							+ SHorizontalBox::Slot()
							[
								SAssignNew(CollectionRenameBox, SEditableTextBox)
								.Text(ViewEntry->EntryLabel)
								
								.Visibility_Lambda([this]()
								{
									if (!bIsEntryBeingRenamed && ViewEntry->bIsRenaming)
									{
										OnCollectionRenameStarted.ExecuteIfBound(ViewEntry, CollectionRenameBox);
									}
									bIsEntryBeingRenamed = ViewEntry->bIsRenaming;
									return ViewEntry->bIsRenaming ? EVisibility::Visible : EVisibility::Collapsed;
								})
								.OnTextCommitted_Lambda([this](const FText & NewText, ETextCommit::Type CommitStatus)
								{
									OnCollectionRenameEnded.ExecuteIfBound(ViewEntry, NewText);
								})
								.Font_Lambda([this]()
								{
									return FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText").Font;							
								})
							]
						]


					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSpacer)
					
					]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Right)
						.Padding(5.0f)
						[
							SNew(STextBlock)
							.Text(TAttribute<FText>::CreateLambda([this]()
							{
								return FText::AsNumber(ViewEntry->Count);
							}))
						]
				];
			}
		}
		END_SLATE_FUNCTION_BUILD_OPTIMIZATION

	private:
		TSharedPtr<SToolPresetManager::FToolPresetViewEntry> ViewEntry;
		TSharedPtr<SCheckBox> EnabledWidget;
		TSharedPtr<SEditableTextBox> CollectionRenameBox;

		bool bIsEntryBeingRenamed = false;

		FOnCollectionEnabledCheckboxChanged OnCollectionEnabledCheckboxChanged;
		FOnCollectionRenameStarted OnCollectionRenameStarted;
		FOnCollectionRenameEnded OnCollectionRenameEnded;
	};

	template<typename ItemType>
	class SToolPresetTableRow : public SMultiColumnTableRow< ItemType>
	{
		typedef SToolPresetManager::FToolPresetViewEntry::EEntryType EEntryType;
		using typename SMultiColumnTableRow< ItemType>::FSuperRowType;

	public:

		SLATE_BEGIN_ARGS(SToolPresetTableRow) { }
		SLATE_ARGUMENT(TSharedPtr<SToolPresetManager::FToolPresetViewEntry>, ViewEntry)
		SLATE_EVENT(FOnPresetDeleted, OnPresetDeleted)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
		{
			typename FSuperRowType::FArguments Args = typename FSuperRowType::FArguments()
				.ExpanderStyleSet(&FCoreStyle::Get());

			OnPresetDeleted = InArgs._OnPresetDeleted;
			ViewEntry = InArgs._ViewEntry;

			SMultiColumnTableRow<ItemType>::Construct(Args, InOwnerTableView);
		}

		BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
		virtual void ConstructChildren(ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent) override
		{
			STableRow<ItemType>::Content = InContent;

			TSharedPtr<class ITableRow> ThisTableRow = this->SharedThis(this);

			if (ViewEntry->EntryType == EEntryType::Tool)
			{
				this->ChildSlot				
				[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(5.0f)
					[
						SNew(SImage)
						.Visibility_Lambda([this]()
						{
							return ViewEntry->EntryType == EEntryType::Tool ? EVisibility::Visible : EVisibility::Collapsed;
						})
						.Image(&ViewEntry->EntryIcon)
					]

				+ SHorizontalBox::Slot()
					.FillWidth(1)
					.Padding(5.0f)
					[
						SNew(STextBlock)
						.Text(ViewEntry->EntryLabel)
					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSpacer)

					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(EHorizontalAlignment::HAlign_Right)
					.Padding(5.0f)
					[
						SNew(STextBlock)
						.Text(TAttribute<FText>::CreateLambda([this]()
						{
							return FText::AsNumber(ViewEntry->Count);
						}))
					]
				];
			}
			else
			{
				ensure(ViewEntry->EntryType == EEntryType::Preset);

				this->ChildSlot
				.Padding(InPadding)
				[
					InContent
				];
			}
			
		}

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName)
		{
			if (InColumnName == "Label")
			{
				return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(FMargin(4.0f, 0.0f))
					[
						SNew(STextBlock)
						.Text(TAttribute<FText>::CreateLambda([this]()
						{
							return FText::FromString(ViewEntry->PresetLabel);
						}))
					];
			}
			else if (InColumnName == "Tooltip")
			{
				return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(FMargin(4.0f, 0.0f))
					[
						SNew(STextBlock)
						.Text(TAttribute<FText>::CreateLambda([this]()
						{
							return FText::FromString(ViewEntry->PresetTooltip);
						}))
					];
			}
			else if (InColumnName == "Tool")
			{
				return SNew(SBox)
					.VAlign(VAlign_Center)
					.Padding(FMargin(5.0f, 5.0f))
					[
						SNew(SImage)
						.Image(&ViewEntry->EntryIcon)
						.DesiredSizeOverride(FVector2D(16, 16))
					];
			}
			else if (InColumnName == "Delete")
			{
				return SNew(SNegativeActionButton)
					.Icon(FAppStyle::GetBrush("Icons.Delete"))						
					.OnClicked_Lambda([this]()
					{
						OnPresetDeleted.ExecuteIfBound(ViewEntry); return FReply::Handled();
					})
					.Visibility_Lambda([this]()
					{
						return this->IsHovered() ? EVisibility::Visible : EVisibility::Hidden;
					});
			}

			return SNullWidget::NullWidget;

		}
		END_SLATE_FUNCTION_BUILD_OPTIMIZATION

	private:
		TSharedPtr<SToolPresetManager::FToolPresetViewEntry> ViewEntry;
		FOnPresetDeleted                     OnPresetDeleted;
	};

}



/* SToolPresetManager interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SToolPresetManager::Construct( const FArguments& InArgs )
{
	UToolPresetUserSettings::Initialize();
	BindCommands();

	UserSettings = UToolPresetUserSettings::Get();
	if (UserSettings.IsValid())
	{
		UserSettings->LoadEditorConfig();
	}

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.FillHeight(1.0)
		//.AutoHeight()
		[
				SAssignNew(Splitter, SSplitter)
					.Orientation(EOrientation::Orient_Horizontal)
					
					+ SSplitter::Slot()
					.Value(0.4f)
					.Resizable(false)
						[
							SNew(SVerticalBox)		

								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(SBorder)
									.BorderImage(&FAppStyle::Get().GetWidgetStyle<FTableColumnHeaderStyle>("TableView.Header.Column").NormalBrush)
									[
										SNew(SHorizontalBox)
									
										+ SHorizontalBox::Slot()
										.AutoWidth()
										.HAlign(HAlign_Right)
										.VAlign(VAlign_Fill)
										[
											SAssignNew(UserCollectionsExpander, SButton)
											.ButtonStyle(FCoreStyle::Get(), "NoBorder" )
											.VAlign(VAlign_Center)
											.HAlign(HAlign_Center)
											.ClickMethod( EButtonClickMethod::MouseDown )
											.OnClicked_Lambda([this]()
											{
												bAreUserCollectionsExpanded = !bAreUserCollectionsExpanded; return FReply::Handled();
											})
											.ContentPadding(0.f)
											.ForegroundColor( FSlateColor::UseForeground() )
											.IsFocusable( false )
											[
												SNew(SImage)
												.Image( this, &SToolPresetManager::GetUserCollectionsExpanderImage )
												.ColorAndOpacity( FSlateColor::UseSubduedForeground() )
											]
										]
									

										+ SHorizontalBox::Slot()
										.FillWidth(1.0)
										.Padding(5.0f, 5.0f, 5.0f, 5.0f)
										.HAlign(HAlign_Left)
										.VAlign(VAlign_Center)
										[
											SNew(STextBlock)										
											.Text(LOCTEXT("UserPresetLabels", "User Preset Collections"))
											.Font(FSlateFontInfo(FCoreStyle::GetDefaultFont(), 13, "Bold"))
										]

										+ SHorizontalBox::Slot()
										.AutoWidth()
										.HAlign(HAlign_Right)
										.VAlign(VAlign_Center)
										.Padding(16.0f, 8.0f, 8.0f, 8.0f)
										[
											SAssignNew(AddUserPresetButton, SPositiveActionButton)
											.ToolTipText(LOCTEXT("AddUserPresetCollection", "Add User Preset Collection"))
											.Icon(FAppStyle::GetBrush("Icons.Plus"))
											.OnClicked_Lambda([this]()
											{
												AddNewUserPresetCollection(); return FReply::Handled();
											})
										]
									]
								]

								+ SVerticalBox::Slot()
								.AutoHeight()
								[
			
											SAssignNew(EditorPresetCollectionTreeView, STreeView<TSharedPtr<FToolPresetViewEntry> >)
												.TreeItemsSource(&EditorCollectionsDataList)
												.SelectionMode(ESelectionMode::Single)
												.OnGenerateRow(this, &SToolPresetManager::HandleTreeGenerateRow)
												.OnGetChildren(this, &SToolPresetManager::HandleTreeGetChildren)
												.OnSelectionChanged(this, &SToolPresetManager::HandleEditorTreeSelectionChanged)
												.Visibility_Lambda([this]()
												{
													return bAreUserCollectionsExpanded ? EVisibility::Visible : EVisibility::Collapsed;
												})
												.HeaderRow
												(
													SNew(SHeaderRow)
													.Visibility(EVisibility::Collapsed)

													+ SHeaderRow::Column("Collection")
													.FixedWidth(150.0f)
													.HeaderContent()
													[
														SNew(STextBlock)
														.Text(LOCTEXT("ToolPresetManagerCollectionTitleHeader", "Collection"))
													]


												)											
								]

								+ SVerticalBox::Slot()
									.FillHeight(1.0f)
								[
			
											SAssignNew(UserPresetCollectionTreeView, STreeView<TSharedPtr<FToolPresetViewEntry> >)
												.TreeItemsSource(&UserCollectionsDataList)
												.SelectionMode(ESelectionMode::Single)
												.OnGenerateRow(this, &SToolPresetManager::HandleTreeGenerateRow)
												.OnGetChildren(this, &SToolPresetManager::HandleTreeGetChildren)
												.OnSelectionChanged(this, &SToolPresetManager::HandleUserTreeSelectionChanged)
												.OnContextMenuOpening(this, &SToolPresetManager::OnGetCollectionContextMenuContent)
												.Visibility_Lambda([this]()
												{
													return bAreUserCollectionsExpanded ? EVisibility::Visible : EVisibility::Collapsed;
												})
												.HeaderRow
												(
													SNew(SHeaderRow)
													.Visibility(EVisibility::Collapsed)

													+ SHeaderRow::Column("Collection")
													.FixedWidth(150.0f)
													.HeaderContent()
													[
														SNew(STextBlock)
														.Text(LOCTEXT("ToolPresetManagerCollectionTitleHeader", "Collection"))
													]


												)											
								]

								+ SVerticalBox::Slot()
									.AutoHeight()									
								[
									SNew(SBorder)
									.BorderImage(&FAppStyle::Get().GetWidgetStyle<FTableColumnHeaderStyle>("TableView.Header.Column").NormalBrush)
									[
										SNew(SHorizontalBox)
										+ SHorizontalBox::Slot()
										.AutoWidth()
										.HAlign(HAlign_Right)
										.VAlign(VAlign_Fill)
										[
											SAssignNew(ProjectCollectionsExpander, SButton)
											.ButtonStyle(FCoreStyle::Get(), "NoBorder" )
											.VAlign(VAlign_Center)
											.HAlign(HAlign_Center)
											.ClickMethod( EButtonClickMethod::MouseDown )
											.OnClicked_Lambda([this]()
											{
												bAreProjectCollectionsExpanded = !bAreProjectCollectionsExpanded; return FReply::Handled();
											})
											.ContentPadding(0.f)
											.ForegroundColor( FSlateColor::UseForeground() )
											.IsFocusable( false )
											[
												SNew(SImage)
												.Image( this, &SToolPresetManager::GetProjectCollectionsExpanderImage )
												.ColorAndOpacity( FSlateColor::UseSubduedForeground() )
											]
										]

										+ SHorizontalBox::Slot()
										.FillWidth(1.0)
										.Padding(5.0f, 5.0f, 5.0f, 5.0f)
										.HAlign(HAlign_Left)
										.VAlign(VAlign_Center)
										[
											SNew(STextBlock)
											.Text(LOCTEXT("ProjectPresetLabels", "Project Preset Collections"))
											.Font(FSlateFontInfo(FCoreStyle::GetDefaultFont(), 13, "Bold"))
										]

										+ SHorizontalBox::Slot()
										.AutoWidth()
										.HAlign(HAlign_Right)
										.VAlign(VAlign_Center)
										.Padding(16.0f, 8.0f, 8.0f, 8.0f)
										[
											SNew(SSimpleButton)
											.ToolTipText(LOCTEXT("OpenProjectSettingsPresets", "Open Project Settings for Presets"))
											.Icon(FAppStyle::GetBrush("Icons.Settings"))
											.OnClicked_Lambda([this]() 
											{ 
												if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
												{
													SettingsModule->ShowViewer("Project", "Plugins", "Interactive Tool Presets");
												}
												return FReply::Handled();
											})
										]
									]
								]

								+ SVerticalBox::Slot()
									.FillHeight(1.0f)
								[
										
											SNew(SVerticalBox)
											.Visibility_Lambda([this]()
											{
												return bAreProjectCollectionsExpanded ? EVisibility::Visible : EVisibility::Collapsed;
											})

											+ SVerticalBox::Slot()
											.FillHeight(1.0f)
											[

											SAssignNew(ProjectPresetCollectionTreeView, STreeView<TSharedPtr<FToolPresetViewEntry> >)
											.Visibility(this, &SToolPresetManager::ProjectPresetCollectionsVisibility)
												.TreeItemsSource(&ProjectCollectionsDataList)
												.SelectionMode(ESelectionMode::Single)
												.OnGenerateRow(this, &SToolPresetManager::HandleTreeGenerateRow)
												.OnGetChildren(this, &SToolPresetManager::HandleTreeGetChildren)
												.OnSelectionChanged(this, &SToolPresetManager::HandleTreeSelectionChanged)
												.OnContextMenuOpening(this, &SToolPresetManager::OnGetCollectionContextMenuContent)
												.HeaderRow
												(
													SNew(SHeaderRow)
													.Visibility(EVisibility::Collapsed)

													+ SHeaderRow::Column("Collection")
													.FixedWidth(150.0f)
													.HeaderContent()
													[
														SNew(STextBlock)
														.Text(LOCTEXT("ToolPresetManagerCollectionTitleHeader", "Collection"))
													]


												)											
											]

											+ SVerticalBox::Slot()
											.AutoHeight()
											.HAlign(HAlign_Center)
											.Padding(5.0f)
											[
												SNew(STextBlock)
												.WrapTextAt(150.0f)												
												.Visibility_Lambda([this]()
												{
													return ProjectPresetCollectionsVisibility() == EVisibility::Visible ? EVisibility::Collapsed : EVisibility::Visible;
												})
												.Text(LOCTEXT("ProjectPresetsNotLoadedLabel", "Manage Project Preset Collections in Project Settings"))
												.Justification(ETextJustify::Center)
												.Font(FAppStyle::GetFontStyle("NormalFontItalic"))
											]


								]

						]

						+ SSplitter::Slot()
						[
							SNew(SVerticalBox)

							+ SVerticalBox::Slot()
							.FillHeight(1.0)
							[
								SNew(SOverlay)
								+ SOverlay::Slot()
								.ZOrder(1)
							[


								SAssignNew(PresetListView, SListView<TSharedPtr<FToolPresetViewEntry>>)
								.ListItemsSource(&PresetDataList)
								.SelectionMode(ESelectionMode::SingleToggle)
								.OnGenerateRow(this, &SToolPresetManager::HandleListGenerateRow)		
								.OnSelectionChanged(this, &SToolPresetManager::HandleListSelectionChanged)
								.OnContextMenuOpening(this, &SToolPresetManager::OnGetPresetContextMenuContent)
								.HeaderRow
								(
									SNew(SHeaderRow)
									.Visibility(EVisibility::Visible)

									+ SHeaderRow::Column("Tool")
									.FixedWidth(30.0f)
									.HeaderContentPadding(FMargin(5.0f, 5.0f))
									.HAlignHeader(EHorizontalAlignment::HAlign_Center)
									.VAlignHeader(EVerticalAlignment::VAlign_Center)
									.HAlignCell(EHorizontalAlignment::HAlign_Center)
									.HeaderContent()									
									[
										SNew(SImage)
										.Image(FToolPresetEditorStyle::Get()->GetBrush("ManagerIcons.Tools"))
										.DesiredSizeOverride(FVector2D(20, 20))
									]
									
									+ SHeaderRow::Column("Label")
									.FillWidth(80.0f)
									.HeaderContentPadding(FMargin(5.0f, 5.0f))
									.HAlignHeader(EHorizontalAlignment::HAlign_Left)
									.VAlignHeader(EVerticalAlignment::VAlign_Center)
									.HeaderContent()
									[
										SNew(STextBlock)
										.Text(LOCTEXT("ToolPresetManagerPresetLabelHeader", "Label"))
									]

									+ SHeaderRow::Column("Tooltip")
									.FillWidth(80.0f)
									.HAlignHeader(EHorizontalAlignment::HAlign_Left)
									.VAlignHeader(EVerticalAlignment::VAlign_Center)
									.HeaderContent()
									[
										SNew(STextBlock)
										.Text(LOCTEXT("ToolPresetManagerPresetTooltipHeader", "Tooltip"))
									]
								)
							]
							
							+ SOverlay::Slot()
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							.ZOrder(2)
								[
									SNew(STextBlock)
									.WrapTextAt(150.0f)
									.Visibility_Lambda([this]()
									{
										return bHasPresetsInCollection ? EVisibility::Collapsed : EVisibility::Visible;
									})
									.Text(LOCTEXT("NoPresetsAvailableLabel", "Add New Presets from any Modeling Tool"))
									.Justification(ETextJustify::Center)
									.Font(FAppStyle::GetFontStyle("NormalFontItalic"))
								]
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(5.0f)
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(5.0f)
							[
								SNew(SHorizontalBox)								
								+ SHorizontalBox::Slot()
							    .FillWidth(1.f)
								.Padding(5.0f)
								.HAlign(EHorizontalAlignment::HAlign_Left)	
								[
									SNew(STextBlock)
									.Text(LOCTEXT("ToolPresetLabelEditLabel", "Label"))
								]
								+ SHorizontalBox::Slot()
								.FillWidth(1.f)								
								.Padding(5.0f)
								[
									SNew(SEditableTextBox)
									.IsEnabled(this, &SToolPresetManager::EditAreaEnabled)
									.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
									.Text_Lambda([this]() 
									{
										if (ActivePresetToEdit)
										{
											return FText::FromString(ActivePresetToEdit->PresetLabel);
										}
										return FText::GetEmpty();
									})
									.OnTextChanged_Lambda([this](const FText& NewText)
									{
										if (ActivePresetToEdit)
										{
											// Cap the number of characters sent out of the text box, so we don't overflow menus and tooltips
											ActivePresetToEdit->PresetLabel = NewText.ToString().Left(255);
										}
									})
									.OnTextCommitted_Lambda([this](const FText& NewText, ETextCommit::Type CommitStatus)
									{
										if (ActivePresetToEdit)
										{
											// Cap the number of characters sent out of the text box, so we don't overflow menus and tooltips
											SetPresetLabel(ActivePresetToEdit, FText::FromString(NewText.ToString().Left(255)));
										}
									})
								]
							]
							+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(5.0f)
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.FillWidth(1.f)
								.Padding(5.0f)
								.HAlign(EHorizontalAlignment::HAlign_Left)
								[
									SNew(STextBlock)
									.Text(LOCTEXT("ToolPresetTooltipEditLabel", "Tooltip"))
								]
								+ SHorizontalBox::Slot()
								.FillWidth(1.f)
								.Padding(5.0f)																
								[
									SNew(SBox)
									.MinDesiredHeight(44.f)
									.MaxDesiredHeight(44.0f)
									[
										SNew(SMultiLineEditableTextBox)		
										.IsEnabled(this, &SToolPresetManager::EditAreaEnabled)
										.AllowMultiLine(false)
										.AutoWrapText(true)
										.WrappingPolicy(ETextWrappingPolicy::DefaultWrapping)
										.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
										.Text_Lambda([this]()
										{
											if (ActivePresetToEdit)
											{
												return FText::FromString(ActivePresetToEdit->PresetTooltip);
											}
											return FText::GetEmpty();
										})
										.OnTextChanged_Lambda([this](const FText& NewText) 
										{
											if (ActivePresetToEdit)
											{
												// Cap the number of characters sent out of the text box, so we don't overflow menus and tooltips
												ActivePresetToEdit->PresetTooltip = NewText.ToString().Left(2048);
											}
										})
										.OnTextCommitted_Lambda([this](const FText& NewText, ETextCommit::Type CommitStatus)
										{
											if (ActivePresetToEdit)
											{
												// Cap the number of characters sent out of the text box, so we don't overflow menus and tooltips
												SetPresetTooltip(ActivePresetToEdit, FText::FromString(NewText.ToString().Left(2048)));
											}
										})
									]
								]
							]
						]						
					]
				]
		];

		RegeneratePresetTrees();
		if (UserCollectionsDataList.Num() == 0)
		{
			bAreUserCollectionsExpanded = false;
		}
		if (ProjectCollectionsDataList.Num() == 0)
		{
			bAreProjectCollectionsExpanded = false;
		}

}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

SToolPresetManager::~SToolPresetManager()
{
}

void SToolPresetManager::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	RegeneratePresetTrees();
}

void SToolPresetManager::RegeneratePresetTrees()
{
	if (!ensure(UserSettings.IsValid()))
	{
		return;
	}

	const UToolPresetProjectSettings* ProjectSettings = GetDefault<UToolPresetProjectSettings>();
	TArray<FSoftObjectPath> AvailablePresetCollections = ProjectSettings->LoadedPresetCollections.Array();
	TArray<FSoftObjectPath> AvailableUserPresetCollections;
	UE::ToolPresetEditor::Private::GetObjectsOfClass<UInteractiveToolsPresetCollectionAsset>(AvailableUserPresetCollections);

	TotalPresetCount = 0;

	auto GenerateSubTree = [this](UInteractiveToolsPresetCollectionAsset* PresetCollection, TSharedPtr<FToolPresetViewEntry> RootEntry)
	{
		TMap<FString, FInteractiveToolPresetStore >::TIterator ToolNameIter = PresetCollection->PerToolPresets.CreateIterator();
		for (; (bool)ToolNameIter; ++ToolNameIter)
		{
			int32 ToolCount = 0;
			for (int32 PresetIndex = 0; PresetIndex < ToolNameIter.Value().NamedPresets.Num(); ++PresetIndex)
			{
				ToolCount += ToolNameIter.Value().NamedPresets[PresetIndex].IsValid() ? 1 : 0;
			}
			if (ToolCount)
			{
				RootEntry->Children.Add(MakeShared<FToolPresetViewEntry>(
					ToolNameIter.Value().ToolLabel,
					ToolNameIter.Value().ToolIcon,
					RootEntry->CollectionPath,
					ToolNameIter.Key(),
					ToolCount));
				RootEntry->Children.Last()->Parent = RootEntry;
				RootEntry->Count += ToolCount;
				TotalPresetCount += ToolCount;
			}
		}

	};

	auto GenerateTreeEntries = [this, &GenerateSubTree](TObjectPtr<UInteractiveToolsPresetCollectionAsset> DefaultCollection,
		TArray<FSoftObjectPath>* AssetList,
		TArray< TSharedPtr< FToolPresetViewEntry > >& TreeList,
		TSharedPtr<STreeView<TSharedPtr<FToolPresetViewEntry> > >& TreeView)
	{
		bool bTreeNeedsRefresh = false;
		TArray< TSharedPtr< FToolPresetViewEntry > > TempTreeDataList;

		if (DefaultCollection)
		{
			TSharedPtr<FToolPresetViewEntry> CollectionEntry = MakeShared<FToolPresetViewEntry>(
				UserSettings->bDefaultCollectionEnabled,
				FSoftObjectPath(),
				DefaultCollection->CollectionLabel,
				0);
			CollectionEntry->bIsDefaultCollection = true;
			GenerateSubTree(DefaultCollection, CollectionEntry);
			TempTreeDataList.Add(CollectionEntry);
		}

		if (AssetList)
		{
			AssetList->RemoveAll([](const FSoftObjectPath& Path)
			{
				return !Path.IsAsset();
			});

			for (const FSoftObjectPath& Path : *AssetList)
			{
				UInteractiveToolsPresetCollectionAsset* PresetCollection = nullptr;

				if (Path.IsAsset())
				{
					PresetCollection = Cast<UInteractiveToolsPresetCollectionAsset>(Path.TryLoad());
				}
				if (PresetCollection)
				{
					TSharedPtr<FToolPresetViewEntry> CollectionEntry = MakeShared<FToolPresetViewEntry>(
						UserSettings->EnabledPresetCollections.Contains(Path),
						Path,
						PresetCollection->CollectionLabel,
						0);
					GenerateSubTree(PresetCollection, CollectionEntry);
					TempTreeDataList.Add(CollectionEntry);
				}
			}
		}

		if (TempTreeDataList.Num() != TreeList.Num())
		{
			bTreeNeedsRefresh = true;
		}
		else
		{
			for (int32 CollectionIndex = 0; CollectionIndex < TreeList.Num(); ++CollectionIndex)
			{
				if (!(TreeList[CollectionIndex]->HasSameMetadata(*TempTreeDataList[CollectionIndex])))
				{
					bTreeNeedsRefresh = true;
				}
			}
		}

		if (bTreeNeedsRefresh)
		{
			TreeList = TempTreeDataList;
			TreeView->RequestTreeRefresh();
			bHasActiveCollection = false;
		}

		for (TSharedPtr<FToolPresetViewEntry>& Entry : TreeList)
		{
			Entry->bEnabled = UserSettings->EnabledPresetCollections.Contains(Entry->CollectionPath);
			if (Entry->bIsDefaultCollection)
			{
				Entry->bEnabled = UserSettings->bDefaultCollectionEnabled;
			}
			else
			{
				Entry->bEnabled = UserSettings->EnabledPresetCollections.Contains(Entry->CollectionPath);
			}
		}
	};

	// Handle the default collection
	UToolPresetAssetSubsystem* PresetAssetSubsystem = GEditor->GetEditorSubsystem<UToolPresetAssetSubsystem>();
	TObjectPtr<UInteractiveToolsPresetCollectionAsset> DefaultCollection = nullptr;
	if (ensure(PresetAssetSubsystem))
	{
		DefaultCollection = PresetAssetSubsystem->GetDefaultCollection();
	}

	GenerateTreeEntries(nullptr, &AvailablePresetCollections, ProjectCollectionsDataList, ProjectPresetCollectionTreeView);
	GenerateTreeEntries(nullptr, &AvailableUserPresetCollections, UserCollectionsDataList, UserPresetCollectionTreeView);
	GenerateTreeEntries(DefaultCollection, nullptr, EditorCollectionsDataList, EditorPresetCollectionTreeView);

}


/* SToolPresetManager implementation
 *****************************************************************************/

int32 SToolPresetManager::GetTotalPresetCount() const
{
	return TotalPresetCount;
}

TSharedRef<ITableRow> SToolPresetManager::HandleTreeGenerateRow(TSharedPtr<FToolPresetViewEntry> TreeEntry, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(UE::ToolPresetEditor::Private::SCollectionTableRow< TSharedPtr<FToolPresetViewEntry> >, OwnerTable)
		.ViewEntry(TreeEntry)
		.OnCollectionEnabledCheckboxChanged(this, &SToolPresetManager::SetCollectionEnabled)
		.OnCollectionRenameStarted(this, &SToolPresetManager::CollectionRenameStarted)
		.OnCollectionRenameEnded(this, &SToolPresetManager::CollectionRenameEnded);
}

void SToolPresetManager::HandleTreeGetChildren(TSharedPtr<FToolPresetViewEntry> TreeEntry, TArray< TSharedPtr<FToolPresetViewEntry> >& ChildrenOut)
{
	ChildrenOut = TreeEntry->Children;
}

void SToolPresetManager::GeneratePresetList(TSharedPtr<FToolPresetViewEntry> TreeEntry)
{
	PresetDataList.Empty();
	PresetListView->RequestListRefresh();
	bHasActiveCollection = false;
	ActivePresetToEdit = nullptr;
	bHasPresetsInCollection = false;

	if (!TreeEntry)
	{
		return;
	}

	if (TreeEntry->EntryType == FToolPresetViewEntry::EEntryType::Collection ||
		TreeEntry->EntryType == FToolPresetViewEntry::EEntryType::Tool)
	{
		UInteractiveToolsPresetCollectionAsset* PresetCollection = GetCollectionFromEntry(TreeEntry);

		if (PresetCollection)
		{
			if (TreeEntry->EntryType == FToolPresetViewEntry::EEntryType::Collection)
			{
				bHasActiveCollection = true;
				bIsActiveCollectionEnabled = TreeEntry->bEnabled;
				ActiveCollectionLabel = TreeEntry->EntryLabel;

				TMap<FString, FInteractiveToolPresetStore >::TIterator ToolNameIter = PresetCollection->PerToolPresets.CreateIterator();
				for (; (bool)ToolNameIter; ++ToolNameIter)
				{
					int32 ToolCount = ToolNameIter.Value().NamedPresets.Num();
					for (int32 PresetIndex = 0; PresetIndex < ToolNameIter.Value().NamedPresets.Num(); ++PresetIndex)
					{
						if (ToolNameIter.Value().NamedPresets[PresetIndex].IsValid())
						{
							bHasPresetsInCollection = true;
							PresetDataList.Add(MakeShared<FToolPresetViewEntry>(
								ToolNameIter.Key(),
								PresetIndex,
								ToolNameIter.Value().NamedPresets[PresetIndex].Label,
								ToolNameIter.Value().NamedPresets[PresetIndex].Tooltip,
								FText::FromString(ToolNameIter.Value().NamedPresets[PresetIndex].Label)
								));
							PresetDataList.Last()->Parent = TreeEntry;
							PresetDataList.Last()->CollectionPath = TreeEntry->CollectionPath;
							PresetDataList.Last()->EntryIcon = ToolNameIter.Value().ToolIcon;
						}
					}
				}
			}
			else
			{
				bHasActiveCollection = true;
				bIsActiveCollectionEnabled = TreeEntry->Parent->bEnabled;
				ActiveCollectionLabel = TreeEntry->Parent->EntryLabel;

				const FInteractiveToolPresetStore* ToolData = PresetCollection->PerToolPresets.Find(TreeEntry->ToolName);
				if (!ToolData)
				{
					return;
				}
				int32 ToolCount = ToolData->NamedPresets.Num();
				for (int32 PresetIndex = 0; PresetIndex < ToolData->NamedPresets.Num(); ++PresetIndex)
				{
					if (ToolData->NamedPresets[PresetIndex].IsValid())
					{
						bHasPresetsInCollection = true;
						PresetDataList.Add(MakeShared<FToolPresetViewEntry>(
							TreeEntry->ToolName,
							PresetIndex,
							ToolData->NamedPresets[PresetIndex].Label,
							ToolData->NamedPresets[PresetIndex].Tooltip,
							FText::FromString(ToolData->NamedPresets[PresetIndex].Label)
							));
						PresetDataList.Last()->Parent = TreeEntry;
						PresetDataList.Last()->CollectionPath = TreeEntry->CollectionPath;
						PresetDataList.Last()->EntryIcon = TreeEntry->EntryIcon;
					}
				}
			}
		}
	}

}

void SToolPresetManager::HandleEditorTreeSelectionChanged(TSharedPtr<FToolPresetViewEntry> TreeEntry, ESelectInfo::Type SelectInfo)
{
	for (TSharedPtr<FToolPresetViewEntry> Entry : UserPresetCollectionTreeView->GetRootItems())
	{
		Entry->bIsRenaming = false;
	}

	if (SelectInfo != ESelectInfo::Direct)
	{
		UserPresetCollectionTreeView->ClearSelection();
		ProjectPresetCollectionTreeView->ClearSelection();
		GeneratePresetList(TreeEntry);

		LastFocusedList = EditorPresetCollectionTreeView;
	}
}

void SToolPresetManager::HandleTreeSelectionChanged(TSharedPtr<FToolPresetViewEntry> TreeEntry, ESelectInfo::Type SelectInfo)
{
	for (TSharedPtr<FToolPresetViewEntry> Entry : UserPresetCollectionTreeView->GetRootItems())
	{
		Entry->bIsRenaming = false;
	}

	if (SelectInfo != ESelectInfo::Direct)
	{
		UserPresetCollectionTreeView->ClearSelection();
		EditorPresetCollectionTreeView->ClearSelection();
		GeneratePresetList(TreeEntry);

		LastFocusedList = ProjectPresetCollectionTreeView;

	}
}

void SToolPresetManager::HandleUserTreeSelectionChanged(TSharedPtr<FToolPresetViewEntry> TreeEntry, ESelectInfo::Type SelectInfo)
{
	for (TSharedPtr<FToolPresetViewEntry> Entry : UserPresetCollectionTreeView->GetRootItems())
	{
		Entry->bIsRenaming = false;
	}

	if (SelectInfo != ESelectInfo::Direct)
	{
		ProjectPresetCollectionTreeView->ClearSelection();
		EditorPresetCollectionTreeView->ClearSelection();
		GeneratePresetList(TreeEntry);

		LastFocusedList = UserPresetCollectionTreeView;

	}
}

TSharedRef<ITableRow> SToolPresetManager::HandleListGenerateRow(TSharedPtr<FToolPresetViewEntry> TreeEntry, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(UE::ToolPresetEditor::Private::SToolPresetTableRow< TSharedPtr<FToolPresetViewEntry> >, OwnerTable)
		.ViewEntry(TreeEntry)
		.OnPresetDeleted(this, &SToolPresetManager::DeletePresetFromCollection);
}

void SToolPresetManager::HandleListSelectionChanged(TSharedPtr<FToolPresetViewEntry> TreeEntry, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		LastFocusedList = PresetListView;
	}


	if (TreeEntry)
	{
		ActivePresetToEdit = TreeEntry;
	}
	else
	{
		if (ActivePresetToEdit)
		{
			SetPresetLabel(ActivePresetToEdit, FText::FromString(ActivePresetToEdit->PresetLabel));
			SetPresetTooltip(ActivePresetToEdit, FText::FromString(ActivePresetToEdit->PresetTooltip));
		}
		ActivePresetToEdit.Reset();
	}
}

bool SToolPresetManager::EditAreaEnabled() const
{
	return ActivePresetToEdit.IsValid();
}

EVisibility SToolPresetManager::ProjectPresetCollectionsVisibility() const
{
	return ProjectCollectionsDataList.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
}

void SToolPresetManager::SetCollectionEnabled(TSharedPtr<FToolPresetViewEntry> TreeEntry, ECheckBoxState State)
{	
	if (!ensure(UserSettings.IsValid()))
	{
		return;
	}
	if (TreeEntry->bIsDefaultCollection)
	{
		UserSettings->bDefaultCollectionEnabled = (State == ECheckBoxState::Checked);
		UserSettings->SaveEditorConfig();
	}
	else
	{
		if (State == ECheckBoxState::Checked && !UserSettings->EnabledPresetCollections.Contains(TreeEntry->CollectionPath))
		{
			UserSettings->EnabledPresetCollections.Add(TreeEntry->CollectionPath);
			UserSettings->SaveEditorConfig();
		}
		else if (State != ECheckBoxState::Checked && UserSettings->EnabledPresetCollections.Contains(TreeEntry->CollectionPath))
		{
			UserSettings->EnabledPresetCollections.Remove(TreeEntry->CollectionPath);
			UserSettings->SaveEditorConfig();
		}
	}
}

void SToolPresetManager::CollectionRenameStarted(TSharedPtr<FToolPresetViewEntry> TreeEntry, TSharedPtr<SEditableTextBox> RenameWidget)
{
	// TODO: Figure out why this crashes
	//FSlateApplication::Get().SetKeyboardFocus(RenameWidget, EFocusCause::SetDirectly);
}

void SToolPresetManager::CollectionRenameEnded(TSharedPtr<FToolPresetViewEntry> TreeEntry, const FText& NewText)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	TreeEntry->bIsRenaming = false;

	FAssetData CollectionAsset;
	if (AssetRegistryModule.Get().TryGetAssetByObjectPath(TreeEntry->CollectionPath, CollectionAsset) == UE::AssetRegistry::EExists::Exists)
	{
		UInteractiveToolsPresetCollectionAsset* CollectionObject = Cast<UInteractiveToolsPresetCollectionAsset>(CollectionAsset.GetAsset());
		if (!CollectionObject)
		{
			return;
		}

		TArray< FAssetRenameData> RenameData;
		
		FString NewPackageName, NewAssetName;
		FString SanitizedName = UPackageTools::SanitizePackageName(NewText.ToString());
		IAssetTools::Get().CreateUniqueAssetName(SanitizedName, "", NewPackageName, NewAssetName);

		RenameData.SetNum(1);		
		RenameData[0].Asset = CollectionObject;
		RenameData[0].NewName = NewAssetName;
		RenameData[0].NewPackagePath = CollectionAsset.PackagePath.ToString();
		if(IAssetTools::Get().RenameAssets(RenameData))
		{
			if (UserSettings->EnabledPresetCollections.Contains(TreeEntry->CollectionPath))
			{
				UserSettings->EnabledPresetCollections.Remove(TreeEntry->CollectionPath);
				UserSettings->EnabledPresetCollections.Add(CollectionObject->GetPathName());
				UserSettings->SaveEditorConfig();
			}

			CollectionObject->CollectionLabel = NewText;
			CollectionObject->MarkPackageDirty();	
		}
	}
}


void SToolPresetManager::DeletePresetFromCollection(TSharedPtr< FToolPresetViewEntry > Entry)
{
	UInteractiveToolsPresetCollectionAsset* PresetCollection = GetCollectionFromEntry(Entry);
	if (PresetCollection)
	{
		PresetCollection->PerToolPresets[Entry->ToolName].NamedPresets.RemoveAt(Entry->PresetIndex);
		PresetCollection->MarkPackageDirty();

		GeneratePresetList(Entry->Parent);
	}

	SaveIfDefaultCollection(Entry);
}

void SToolPresetManager::SetPresetLabel(TSharedPtr< FToolPresetViewEntry > Entry, FText InLabel)
{
	UInteractiveToolsPresetCollectionAsset* PresetCollection = GetCollectionFromEntry(Entry);
	if (PresetCollection)
	{
		PresetCollection->PerToolPresets[Entry->ToolName].NamedPresets[Entry->PresetIndex].Label = InLabel.ToString();
		PresetCollection->MarkPackageDirty();
	}

	SaveIfDefaultCollection(Entry);
}

void SToolPresetManager::SetPresetTooltip(TSharedPtr< FToolPresetViewEntry > Entry, FText InTooltip)
{
	UInteractiveToolsPresetCollectionAsset* PresetCollection = GetCollectionFromEntry(Entry);
	if (PresetCollection)
	{
		PresetCollection->PerToolPresets[Entry->ToolName].NamedPresets[Entry->PresetIndex].Tooltip = InTooltip.ToString();
		PresetCollection->MarkPackageDirty();
	}

	SaveIfDefaultCollection(Entry);
}

void SToolPresetManager::DeleteSelectedUserPresetCollection()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	TArray<TSharedPtr<FToolPresetViewEntry>> SelectedUserCollections = UserPresetCollectionTreeView->GetSelectedItems();

	if (SelectedUserCollections.Num() == 1)
	{
		TSharedPtr<FToolPresetViewEntry> Entry = SelectedUserCollections[0];
		if (Entry->bIsDefaultCollection)
		{
			return;
		}

		FAssetData CollectionAsset;
		if (AssetRegistryModule.Get().TryGetAssetByObjectPath(Entry->CollectionPath, CollectionAsset) == UE::AssetRegistry::EExists::Exists)
		{
			TArray<FAssetData> AssetData;
			AssetData.Add(CollectionAsset);
			ObjectTools::DeleteAssets(AssetData, true);
		}

		GeneratePresetList(nullptr);
	}
}

void SToolPresetManager::AddNewUserPresetCollection()
{
	// Load necessary modules
	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	// Generate a unique asset name
	FString Name, PackageName;
	AssetToolsModule.Get().CreateUniqueAssetName(TEXT("/ToolPresets/Presets/"), TEXT("UserPresetCollection"), PackageName, Name);
	const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);

	// Create object and package
	UPackage* package = CreatePackage(*PackageName);
	UInteractiveToolsPresetCollectionAssetFactory* MyFactory = NewObject<UInteractiveToolsPresetCollectionAssetFactory>(UInteractiveToolsPresetCollectionAssetFactory::StaticClass()); // Can omit, and a default factory will be used
	UObject* NewObject = AssetToolsModule.Get().CreateAsset(Name, PackagePath, UInteractiveToolsPresetCollectionAsset::StaticClass(), MyFactory);
	UInteractiveToolsPresetCollectionAsset* NewCollection = ExactCast<UInteractiveToolsPresetCollectionAsset>(NewObject);
	NewCollection->CollectionLabel = FText::FromString(Name);
	FSavePackageArgs SavePackageArgs;
	SavePackageArgs.TopLevelFlags = RF_Public | RF_Standalone;
	UPackage::Save(package, NewObject, *FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension()), SavePackageArgs);

	// Inform asset registry
	AssetRegistry.AssetCreated(NewObject);

	// Since we're adding a new entry, open this tree view up again.
	bAreUserCollectionsExpanded = true;
}

const FSlateBrush* SToolPresetManager::GetProjectCollectionsExpanderImage() const
{
	return GetExpanderImage(ProjectCollectionsExpander, false);
}

const FSlateBrush* SToolPresetManager::GetUserCollectionsExpanderImage() const
{
	return GetExpanderImage(UserCollectionsExpander, true);
}

const FSlateBrush* SToolPresetManager::GetExpanderImage(TSharedPtr<SWidget> ExpanderWidget, bool bIsUserCollections) const
{
	const bool bIsItemExpanded = bIsUserCollections ? bAreUserCollectionsExpanded : bAreProjectCollectionsExpanded;

	FName ResourceName;
	if (bIsItemExpanded)
	{
		if (ExpanderWidget->IsHovered())
		{
			static FName ExpandedHoveredName = "TreeArrow_Expanded_Hovered";
			ResourceName = ExpandedHoveredName;
		}
		else
		{
			static FName ExpandedName = "TreeArrow_Expanded";
			ResourceName = ExpandedName;
		}
	}
	else
	{
		if (ExpanderWidget->IsHovered())
		{
			static FName CollapsedHoveredName = "TreeArrow_Collapsed_Hovered";
			ResourceName = CollapsedHoveredName;
		}
		else
		{
			static FName CollapsedName = "TreeArrow_Collapsed";
			ResourceName = CollapsedName;
		}
	}

	return FCoreStyle::Get().GetBrush(ResourceName);
}

UInteractiveToolsPresetCollectionAsset* SToolPresetManager::GetCollectionFromEntry(TSharedPtr<FToolPresetViewEntry> Entry)
{
	UInteractiveToolsPresetCollectionAsset* PresetCollection = nullptr;
	UToolPresetAssetSubsystem* PresetAssetSubsystem = GEditor->GetEditorSubsystem<UToolPresetAssetSubsystem>();
	
	if (Entry->Root().bIsDefaultCollection && ensure(PresetAssetSubsystem))
	{
		PresetCollection = PresetAssetSubsystem->GetDefaultCollection();
	}
	else
	{
		if (Entry->CollectionPath.IsAsset())
		{
			PresetCollection = Cast<UInteractiveToolsPresetCollectionAsset>(Entry->CollectionPath.TryLoad());
		}
	}

	return PresetCollection;
}

void SToolPresetManager::SaveIfDefaultCollection(TSharedPtr<FToolPresetViewEntry> Entry)
{
	UToolPresetAssetSubsystem* PresetAssetSubsystem = GEditor->GetEditorSubsystem<UToolPresetAssetSubsystem>();

	if (Entry->Root().bIsDefaultCollection && ensure(PresetAssetSubsystem))
	{
		ensure(PresetAssetSubsystem->SaveDefaultCollection());
	}
}

TSharedPtr<SWidget> SToolPresetManager::OnGetPresetContextMenuContent() const
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, UICommandList);

	MenuBuilder.BeginSection("ToolPresetManagerPresetAction", LOCTEXT("ToolPresetAction", "Tool Preset Actions"));

	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete, NAME_None, LOCTEXT("DeleteToolPresetLabel", "Delete Tool Preset"), LOCTEXT("DeleteToolPresetToolTip", "Delete the selected tool preset"));

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedPtr<SWidget> SToolPresetManager::OnGetCollectionContextMenuContent() const
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, UICommandList);

	MenuBuilder.BeginSection("ToolPresetManagerCollectionAction", LOCTEXT("CollectionAction", "Tool Preset Collection Actions"));

	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete, NAME_None, LOCTEXT("DeleteCollectionLabel", "Delete Collection"), LOCTEXT("DeleteCollectionToolTip", "Delete the selected collection"));
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename, NAME_None, LOCTEXT("RenameCollectionLabel", "Rename Collection"), LOCTEXT("RenameCollectionToolTip", "Rename the selected collection"));

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SToolPresetManager::BindCommands()
{
	// This should not be called twice on the same instance
	check(!UICommandList.IsValid());

	UICommandList = MakeShareable(new FUICommandList);

	FUICommandList& CommandList = *UICommandList;

	// ...and bind them all

	CommandList.MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SToolPresetManager::OnDeleteClicked),
		FCanExecuteAction::CreateSP(this, &SToolPresetManager::CanDelete));

	CommandList.MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &SToolPresetManager::OnRenameClicked),
		FCanExecuteAction::CreateSP(this, &SToolPresetManager::CanRename));	
}


void SToolPresetManager::OnDeleteClicked()
{
	if (UserPresetCollectionTreeView == LastFocusedList)
	{
		DeleteSelectedUserPresetCollection();
	}

	if (PresetListView == LastFocusedList)
	{
		for (TSharedPtr<FToolPresetViewEntry> Entry : PresetListView->GetSelectedItems())
		{
			DeletePresetFromCollection(Entry);
		}
	}
}

bool SToolPresetManager::CanDelete()
{
	bool bIsListValid = LastFocusedList.IsValid() && (LastFocusedList == UserPresetCollectionTreeView || LastFocusedList == PresetListView);
	bool bIsSelectionValid = false;
	if (bIsListValid)
	{
		bIsSelectionValid = LastFocusedList.Pin()->GetNumItemsSelected() == 1 &&
			                (LastFocusedList.Pin()->GetSelectedItems()[0]->EntryType == FToolPresetViewEntry::EEntryType::Collection ||
							 LastFocusedList.Pin()->GetSelectedItems()[0]->EntryType == FToolPresetViewEntry::EEntryType::Preset);
	}
	return bIsSelectionValid;
}

void SToolPresetManager::OnRenameClicked()
{
	for (TSharedPtr<FToolPresetViewEntry> Entry : UserPresetCollectionTreeView->GetRootItems())
	{
		Entry->bIsRenaming = false;
	}

	for (TSharedPtr<FToolPresetViewEntry> Entry : UserPresetCollectionTreeView->GetSelectedItems())
	{
		Entry->bIsRenaming = true;
	}
}

bool SToolPresetManager::CanRename()
{
	bool bIsListValid = LastFocusedList.IsValid() && LastFocusedList == UserPresetCollectionTreeView;
	bool bIsSelectionValid = false;
	if (bIsListValid)
	{
		bIsSelectionValid = LastFocusedList.Pin()->GetNumItemsSelected() == 1 &&
			                LastFocusedList.Pin()->GetSelectedItems()[0]->EntryType == FToolPresetViewEntry::EEntryType::Collection;
	}
	return bIsSelectionValid;
}

#undef LOCTEXT_NAMESPACE
