// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

class FTransaction;
class FUICommandList;
class SEditableTextBox;
class SNegativeActionButton;
class SPositiveActionButton;
class SSplitter;
class SVerticalBox;
class UInteractiveToolsPresetCollectionAsset;
class UToolPresetUserSettings;
template<class ItemType> class SListView;
template<class ItemType> class STreeView;

/**
 * Implements the preset manager panel.
 */
class SToolPresetManager
	: public SCompoundWidget
{
	struct FToolPresetCollectionInfo
	{
		FSoftObjectPath PresetCollectionPath;
		bool bCollectionEnabled;

		FToolPresetCollectionInfo(FSoftObjectPath InPresetCollectionPath, bool bInEnabled)
			: PresetCollectionPath(InPresetCollectionPath), bCollectionEnabled(bInEnabled)
		{}
	};

public:
	struct FToolPresetViewEntry
	{
		enum class EEntryType : uint8
		{
			Collection,
			Tool,
			Preset
		};
		
		EEntryType EntryType;

		// Used for Collections/Tool Entries
		bool bEnabled = false;
		FSoftObjectPath CollectionPath;
		bool bIsDefaultCollection = false;
		bool bIsRenaming = false;
		FText EntryLabel;
		FSlateBrush EntryIcon;
		int32 Count = 0;

		// Used for Preset/Tool entries
		FString ToolName = "";
		int32 PresetIndex = 0;
		FString PresetLabel = "";
		FString PresetTooltip = "";

		TSharedPtr< FToolPresetViewEntry> Parent;
		TArray<TSharedPtr< FToolPresetViewEntry> > Children;

		// Collection Constructor
		FToolPresetViewEntry(bool bEnabledIn, FSoftObjectPath CollectionPathIn, FText EntryLabelIn, int32 CountIn)
			: EntryType(EEntryType::Collection),
			  bEnabled(bEnabledIn), 
			  CollectionPath(CollectionPathIn),
			  EntryLabel(EntryLabelIn),
			  Count(CountIn)
		{}

		// Tool Constructor
		FToolPresetViewEntry(FText EntryLabelIn, FSlateBrush EntryIconIn, FSoftObjectPath CollectionPathIn, FString ToolNameIn, int32 CountIn)
		  :	EntryType(EEntryType::Tool),
			CollectionPath(CollectionPathIn),
			EntryLabel(EntryLabelIn),
			EntryIcon(EntryIconIn),
			Count(CountIn),
			ToolName(ToolNameIn)
		{}

		// Preset Constructor
		FToolPresetViewEntry(FString ToolNameIn, int32 PresetIndexIn, FString PresetLabelIn, FString PresetTooltipIn, FText EntryLabelIn)
			: EntryType(EEntryType::Preset),
			  EntryLabel(EntryLabelIn),
			  ToolName(ToolNameIn),
			  PresetIndex(PresetIndexIn),
			  PresetLabel(PresetLabelIn),
			  PresetTooltip(PresetTooltipIn)
		{}

		bool HasSameMetadata(FToolPresetViewEntry& Other)
		{
			bool bIsEqual = 
				 EntryType == Other.EntryType &&
				 CollectionPath == Other.CollectionPath &&
				 bIsDefaultCollection == Other.bIsDefaultCollection &&
				 Count == Other.Count &&
				 ToolName.Equals(Other.ToolName) &&
				 PresetIndex == Other.PresetIndex &&
				 PresetLabel.Equals(Other.PresetLabel) &&
				 PresetTooltip.Equals(Other.PresetTooltip) &&
				 Children.Num() == Other.Children.Num();

			for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
			{
				bIsEqual = bIsEqual && (*Children[ChildIndex]).HasSameMetadata(*Other.Children[ChildIndex]);
			}
			return bIsEqual;
		}

		bool operator==(FToolPresetViewEntry& Other)
		{
			bool bIsEqual = bEnabled == Other.bEnabled &&
				 EntryType == Other.EntryType &&
				 CollectionPath == Other.CollectionPath &&
				 bIsDefaultCollection == Other.bIsDefaultCollection && 
				 Count == Other.Count &&
				 EntryLabel.EqualTo(Other.EntryLabel) &&
				 ToolName.Equals(Other.ToolName) &&
				 PresetIndex == Other.PresetIndex &&
				 PresetLabel.Equals(Other.PresetLabel) &&
				 PresetTooltip.Equals(Other.PresetTooltip) &&
				 Children.Num() == Other.Children.Num();

			for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
			{
				bIsEqual = bIsEqual && (*Children[ChildIndex]) == (*Other.Children[ChildIndex]);
			}

			return bIsEqual;
		}

		FToolPresetViewEntry& Root()
		{
			FToolPresetViewEntry* ActiveNode = this;
			while (ActiveNode->Parent)
			{
				ActiveNode = ActiveNode->Parent.Get();
			}
			return *ActiveNode;
		}
	};

	SLATE_BEGIN_ARGS(SToolPresetManager) { }
	SLATE_END_ARGS()

	virtual ~SToolPresetManager();

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	void Construct( const FArguments& InArgs );
	
	//~ Begin SWidget Interface
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	//~ End SWidget Interface

private:
	void BindCommands();

	void RegeneratePresetTrees();

	int32 GetTotalPresetCount() const;

	TSharedRef<ITableRow> HandleTreeGenerateRow(TSharedPtr<FToolPresetViewEntry> TreeEntry, const TSharedRef<STableViewBase>& OwnerTable);
	void HandleTreeGetChildren(TSharedPtr<FToolPresetViewEntry> TreeEntry, TArray< TSharedPtr<FToolPresetViewEntry> >& ChildrenOut);
	void HandleTreeSelectionChanged(TSharedPtr<FToolPresetViewEntry> TreeEntry, ESelectInfo::Type);
	void HandleUserTreeSelectionChanged(TSharedPtr<FToolPresetViewEntry> TreeEntry, ESelectInfo::Type);
	void HandleEditorTreeSelectionChanged(TSharedPtr<FToolPresetViewEntry> TreeEntry, ESelectInfo::Type);
	EVisibility ProjectPresetCollectionsVisibility() const;

	TSharedPtr<SWidget> OnGetPresetContextMenuContent() const;
	TSharedPtr<SWidget> OnGetCollectionContextMenuContent() const;

	void GeneratePresetList(TSharedPtr<FToolPresetViewEntry> TreeEntry);
	TSharedRef<ITableRow> HandleListGenerateRow(TSharedPtr<FToolPresetViewEntry> TreeEntry, const TSharedRef<STableViewBase>& OwnerTable);
    void HandleListSelectionChanged(TSharedPtr<FToolPresetViewEntry> TreeEntry, ESelectInfo::Type SelectInfo);

	bool EditAreaEnabled() const;
	void SetCollectionEnabled(TSharedPtr<FToolPresetViewEntry> TreeEntry, ECheckBoxState State);
	void DeletePresetFromCollection(TSharedPtr< FToolPresetViewEntry > Entry);

	void CollectionRenameStarted(TSharedPtr<FToolPresetViewEntry> TreeEntry, TSharedPtr<SEditableTextBox> RenameWidget);
	void CollectionRenameEnded(TSharedPtr<FToolPresetViewEntry> TreeEntry, const FText& NewText);

	void DeleteSelectedUserPresetCollection();
	void AddNewUserPresetCollection();

	void SetPresetLabel(TSharedPtr< FToolPresetViewEntry >, FText InLabel);
	void SetPresetTooltip(TSharedPtr< FToolPresetViewEntry >, FText InTooltip);

	const FSlateBrush* GetProjectCollectionsExpanderImage() const;
	const FSlateBrush* GetUserCollectionsExpanderImage() const;
	const FSlateBrush* GetExpanderImage(TSharedPtr<SWidget> ExpanderWidget, bool bIsUserCollections) const;

	UInteractiveToolsPresetCollectionAsset* GetCollectionFromEntry(TSharedPtr<FToolPresetViewEntry> Entry);
	void SaveIfDefaultCollection(TSharedPtr<FToolPresetViewEntry> Entry);

	void OnDeleteClicked();
	bool CanDelete();

	void OnRenameClicked();
	bool CanRename();

private:	
	TSharedPtr<FUICommandList> UICommandList;

	TWeakObjectPtr<UToolPresetUserSettings> UserSettings;

	TWeakPtr< SListView<TSharedPtr<FToolPresetViewEntry> > >  LastFocusedList;

	bool bAreProjectCollectionsExpanded = true;
	TSharedPtr<SButton> ProjectCollectionsExpander;
	TArray< TSharedPtr< FToolPresetViewEntry > > ProjectCollectionsDataList;
	TSharedPtr<STreeView<TSharedPtr<FToolPresetViewEntry> > > ProjectPresetCollectionTreeView;

	bool bAreUserCollectionsExpanded = true;
	TSharedPtr<SButton> UserCollectionsExpander;
	TArray< TSharedPtr< FToolPresetViewEntry > > UserCollectionsDataList;
	TSharedPtr<STreeView<TSharedPtr<FToolPresetViewEntry> > > UserPresetCollectionTreeView;

	TArray< TSharedPtr< FToolPresetViewEntry > > EditorCollectionsDataList;
	TSharedPtr<STreeView<TSharedPtr<FToolPresetViewEntry> > > EditorPresetCollectionTreeView;


	TArray< TSharedPtr< FToolPresetViewEntry > > PresetDataList;
	TSharedPtr<SListView<TSharedPtr<FToolPresetViewEntry> > > PresetListView;

	int32 TotalPresetCount = 0;
	bool bHasActiveCollection = false;
	bool bHasPresetsInCollection = false;
	FText ActiveCollectionLabel;
	bool bIsActiveCollectionEnabled = false;

	TSharedPtr<SSplitter> Splitter;

	TSharedPtr<SVerticalBox>  EditPresetArea;
	TSharedPtr<SEditableTextBox> EditPresetLabel;
	TSharedPtr<SEditableTextBox> EditPresetTooltip;
	TSharedPtr<FToolPresetViewEntry> ActivePresetToEdit;

	TSharedPtr<SPositiveActionButton> AddUserPresetButton;
	TSharedPtr<SNegativeActionButton> DeleteUserPresetButton;
};
