// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

TOOLPRESETEDITOR_API extern const FName PresetEditorTabName;

class IToolPresetEditorModule : public IModuleInterface
{
public:
	static IToolPresetEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IToolPresetEditorModule>("ToolPresetEditor");
	}

	virtual void ExecuteOpenPresetEditor() = 0;
};
