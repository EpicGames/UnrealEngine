// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorConfigBase.h"
#include "Engine/DeveloperSettings.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "ToolPresetSettings.generated.h"

#define UE_API TOOLPRESETEDITOR_API

/**
 * Implements the settings for the PresetEditor.
 */
UCLASS(MinimalAPI, EditorConfig = "UToolPresetUserSettings")
class UToolPresetUserSettings : public UEditorConfigBase
{
	GENERATED_BODY()
	
public:
	UPROPERTY(EditAnywhere, Category = "Interactive Tool Presets|Collections", meta = (EditorConfig, AllowedClasses = "/Script/ToolPresetAsset.InteractiveToolsPresetCollectionAsset"))
	TSet<FSoftObjectPath> EnabledPresetCollections;

	//~ Ideally the above property would be able to store if the default collection was enabled or not.
	//~ 
	//~ However, the default collection itself is stored via an alternative JSON representation, accessed through the PresetAssetSubsystem,
	//~ to avoid issues with automatic asset generation. Therefore it doesn't have a "path" in the traditional sense that the other collections
	//~ do, requiring a separate tracking of it's enabled/disabled status.

	UPROPERTY(EditAnywhere, Category = "Interactive Tool Presets|Collections", meta = (EditorConfig))
	bool bDefaultCollectionEnabled = true;

	static UE_API void Initialize();
	static UE_API UToolPresetUserSettings* Get();

private:
	static UE_API TObjectPtr<UToolPresetUserSettings> Instance;
};


/**
 * Implements the settings for the Tool Project Preset Collections.
 */
UCLASS(MinimalAPI, config = Editor)
class UToolPresetProjectSettings
	: public UDeveloperSettings
{
	GENERATED_BODY()
	
public:
	// UDeveloperSettings overrides

	virtual FName GetContainerName() const override { return FName("Project"); }
	virtual FName GetCategoryName() const override { return FName("Plugins"); }
	virtual FName GetSectionName() const override { return FName("Interactive Tool Presets"); }

	virtual FText GetSectionText() const override { return NSLOCTEXT("ToolPresetSettings", "SectionText", "Interactive Tool Presets"); };
	virtual FText GetSectionDescription() const override { return NSLOCTEXT("ToolPresetSettings", "SectionDescription", "Manage preset settings at the project level."); };

public:
	/* Controls which preset collection assets are to be loaded for this project.  */
	UPROPERTY(config, EditAnywhere, Category = "Interactive Tool Presets|Collections", meta = (AllowedClasses = "/Script/ToolPresetAsset.InteractiveToolsPresetCollectionAsset"))
	TSet<FSoftObjectPath> LoadedPresetCollections;
};

#undef UE_API
