{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Tool Presets",
	"Description": "Adds support for saving and loading tool settings as presets.",
	"Category": "Editor",
	"CreatedBy": "Epic Games Inc",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"IsExperimentalVersion": true,
	"Modules": [
		{
			"Name": "ToolPresetAsset",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ToolPresetEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}

	]
}
